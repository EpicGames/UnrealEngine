// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "SlateFileDialogs.h"
#include "SlateFileDialogsStyles.h"
#include "SlateFileDlgWindow.h"

void FSlateFileDialogsModule::StartupModule()
{
	SlateFileDialog = new FSlateFileDialogsModule();

	FileDialogStyle = new FSlateFileDialogsStyle;
	FileDialogStyle->Initialize();
}


void FSlateFileDialogsModule::ShutdownModule()
{
	if (SlateFileDialog != NULL)
	{
		FileDialogStyle->Shutdown();
		delete FileDialogStyle;

		delete SlateFileDialog;
		SlateFileDialog = NULL;
	}
}


bool FSlateFileDialogsModule::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex)
{
	FSlateFileDlgWindow dialog(FileDialogStyle);
	return dialog.OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, OutFilterIndex, DefaultFilterIndex);
}


bool FSlateFileDialogsModule::OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		FString& OutFoldername)
{
	FSlateFileDlgWindow dialog(FileDialogStyle);
	return dialog.OpenDirectoryDialog(ParentWindowHandle, DialogTitle, DefaultPath, OutFoldername);
}


bool FSlateFileDialogsModule::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex)
{
	FSlateFileDlgWindow dialog(FileDialogStyle);
	return dialog.SaveFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, OutFilterIndex, DefaultFilterIndex);
}


ISlateFileDialogsModule* FSlateFileDialogsModule::Get()
{
	return SlateFileDialog;
}

IMPLEMENT_MODULE(FSlateFileDialogsModule, SlateFileDialogs);
