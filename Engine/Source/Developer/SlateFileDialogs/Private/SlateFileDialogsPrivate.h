// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define HIDE_HIDDEN_FILES 1
#define MAX_FILTER_LENGTH 8192
#define MAX_PATH_LENGTH 8192

#define ENABLE_DIRECTORY_WATCHER 1
