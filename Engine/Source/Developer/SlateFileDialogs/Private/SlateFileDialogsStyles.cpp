// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateFileDialogsStyles.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr< FSlateStyleSet > FSlateFileDialogsStyle::StyleInstance = nullptr;

void FSlateFileDialogsStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}


void FSlateFileDialogsStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}


FName FSlateFileDialogsStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("SlateFileDialogsStyle"));
	return StyleSetName;
}


const FSlateStyleSet *FSlateFileDialogsStyle::Get()
{
	return StyleInstance.Get();
}


TSharedPtr< FSlateStyleSet > FSlateFileDialogsStyle::Create()
{
	TSharedPtr< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet(FSlateFileDialogsStyle::GetStyleSetName()));
	Style->SetContentRoot(FPaths::EngineContentDir());

	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);	
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FSlateColor InvertedForeground(FLinearColor(0.0f, 0.0f, 0.0f));
	const FSlateColor SelectionColor(FLinearColor(0.701f, 0.225f, 0.003f));
	const FSlateColor SelectionColor_Inactive(FLinearColor(0.25f, 0.25f, 0.25f));
	const FSlateColor SelectionColor_Pressed(FLinearColor(0.701f, 0.225f, 0.003f));


	// SButton defaults...
	const FButtonStyle Button = FButtonStyle()
		.SetNormal( BOX_BRUSH( "Slate/Common/Button", FVector2D(32,32), 8.0f/32.0f ) )
		.SetHovered( BOX_BRUSH( "Slate/Common/Button_Hovered", FVector2D(32,32), 8.0f/32.0f ) )
		.SetPressed( BOX_BRUSH( "Slate/Common/Button_Pressed", FVector2D(32,32), 8.0f/32.0f ) )
		.SetDisabled( BOX_BRUSH( "Slate/Common/Button_Disabled", 8.0f/32.0f ) )
		.SetNormalPadding( FMargin( 2,2,2,2 ) )
		.SetPressedPadding( FMargin( 2,3,2,1 ) );
	{
		Style->Set( "Button", Button );

		Style->Set( "InvertedForeground", InvertedForeground );
	}

	Style->Set("SlateFileDialogs.Dialog", DEFAULT_FONT("Regular", 10));
	Style->Set("SlateFileDialogs.DialogBold", DEFAULT_FONT("Bold", 10));
	Style->Set("SlateFileDialogs.DialogLarge", DEFAULT_FONT("Bold", 16));
	Style->Set("SlateFileDialogs.DirectoryItem", DEFAULT_FONT("Bold", 11));
	Style->Set( "SlateFileDialogs.GroupBorder", new BOX_BRUSH( "Slate/Common/GroupBorder", FMargin(4.0f/16.0f) ) );

	Style->Set("SlateFileDialogs.Folder16", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_file_folder_16x", Icon16x16));
	Style->Set("SlateFileDialogs.Folder24", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_file_folder_40x", Icon24x24));
	Style->Set("SlateFileDialogs.NewFolder24", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_new_folder_40x", Icon24x24));
	Style->Set("SlateFileDialogs.BrowseBack24", new IMAGE_BRUSH("SlateFileDialogs/Common/back_arrow_40x", Icon24x24));
	Style->Set("SlateFileDialogs.BrowseForward24", new IMAGE_BRUSH("SlateFileDialogs/Common/forward_arrow_40x", Icon24x24));
	Style->Set("SlateFileDialogs.WhiteBackground", new IMAGE_BRUSH("SlateFileDialogs/Common/Window/WindowWhite", Icon64x64));

	Style->Set("SlateFileDialogs.UAsset16", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_uasset_24x", Icon16x16));
	Style->Set("SlateFileDialogs.UProject16", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_uproject_24x", Icon16x16));
	Style->Set("SlateFileDialogs.Model3D", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_model_24x", Icon16x16));
	Style->Set("SlateFileDialogs.Video", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_video_24x", Icon16x16));
	Style->Set("SlateFileDialogs.Audio", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_audio_24x", Icon16x16));
	Style->Set("SlateFileDialogs.Image", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_image_24x", Icon16x16));
	Style->Set("SlateFileDialogs.TextFile", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_text_24x", Icon16x16));
	Style->Set("SlateFileDialogs.PlaceHolder", new IMAGE_BRUSH("SlateFileDialogs/Icons/icon_skull_16x", Icon16x16));


	Style->Set( "SlateFileDialogs.PathDelimiter", new IMAGE_BRUSH( "SlateFileDialogs/Common/SmallArrowRight", Icon10x10 ) );

	Style->Set( "SlateFileDialogs.PathText", FTextBlockStyle()
		.SetFont( DEFAULT_FONT( "Bold", 11 ) )
		.SetColorAndOpacity( FLinearColor( 1.0f, 1.0f, 1.0f ) )
		.SetHighlightColor( FLinearColor( 1.0f, 1.0f, 1.0f ) )
		.SetHighlightShape(BOX_BRUSH("Slate/Common/TextBlockHighlightShape", FMargin(3.f /8.f)))
		.SetShadowOffset( FVector2D( 1,1 ) )
		.SetShadowColorAndOpacity( FLinearColor(0,0,0,0.9f) ) );

	Style->Set("SlateFileDialogs.FlatButton", FButtonStyle(Button)
		.SetNormal(FSlateNoResource())
		.SetHovered(BOX_BRUSH("SlateFileDialogs/Common/FlatButton", 2.0f / 8.0f, SelectionColor))
		.SetPressed(BOX_BRUSH("SlateFileDialogs/Common/FlatButton", 2.0f / 8.0f, SelectionColor_Pressed))
		);

	return Style;	
}

#undef RootToContentDir
