// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateFileDlgWindow.h"
#include "SlateFileDialogsPrivate.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "DirectoryWatcherModule.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

#define LOCTEXT_NAMESPACE "SlateFileDialogsNamespace"

DEFINE_LOG_CATEGORY_STATIC(LogSlateFileDialogs, Log, All);

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------

class FSlateFileDialogVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	FSlateFileDialogVisitor(TArray<TSharedPtr<FFileEntry>> &InFileList,
						TArray<TSharedPtr<FFileEntry>> &InFolderList, const FString& InFilterList)
		: FileList(InFileList),
		FolderList(InFolderList)
	{
		// Process the filters once rather than once for each file encountered
		InFilterList.ParseIntoArray(FilterList, TEXT(";"), true);
		// Remove cruft from the extension list
		for (int32 Index = 0; Index < FilterList.Num(); Index++)
		{
			FilterList[Index].ReplaceInline(TEXT(")"), TEXT(""));
			FilterList[Index] = FilterList[Index].TrimQuotes().TrimStartAndEnd();
		}
	}
	

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		int32 i;

		// break filename from path
		for (i = FCString::Strlen(FilenameOrDirectory) - 1; i >= 0; i--)
		{
			if (FilenameOrDirectory[i] == TCHAR('/'))
			{
				break;
			}
		}

#if HIDE_HIDDEN_FILES
		if (FilenameOrDirectory[i + 1] == TCHAR('.'))
		{
			return true;
		}
#endif

		FDateTime stamp = IFileManager::Get().GetTimeStamp(FilenameOrDirectory);
		FString ModDate = "";		
		FString FileSize = "";

		if (bIsDirectory)
		{
			FolderList.Add(MakeShareable(new FFileEntry(FString(&FilenameOrDirectory[i + 1]), ModDate, FileSize, true)));
		}
		else
		{
			if (PassesFilterTest(&FilenameOrDirectory[i + 1]))
			{
				int64 size = IFileManager::Get().FileSize(FilenameOrDirectory);

				if (size < 1048576)
				{
					size = (size + 1023) / 1024;
					FileSize = FString::FromInt(size) + " KB";
				}
				else
				{
					size /= 1024;

					if (size < 1048576)
					{
						size = (size + 1023) / 1024;
						FileSize = FString::FromInt(size) + " MB";
					}
					else
					{
						size /= 1024;

						size = (size + 1023) / 1024;
						FileSize = FString::FromInt(size) + " GB";
					}
				}
				
				
				ModDate = FString::Printf(TEXT("%02d/%02d/%04d "), stamp.GetMonth(), stamp.GetDay(), stamp.GetYear());
				
				if (stamp.GetHour() == 0)
				{
					ModDate = ModDate + FString::Printf(TEXT("12:%02d AM"), stamp.GetMinute());
				}
				else if (stamp.GetHour() < 12)
				{
					ModDate = ModDate + FString::Printf(TEXT("%2d:%02d AM"), stamp.GetHour12(), stamp.GetMinute());
				}
				else
				{
					ModDate = ModDate + FString::Printf(TEXT("%2d:%02d PM"), stamp.GetHour12(), stamp.GetMinute());
				}

				FileList.Add(MakeShareable(new FFileEntry(FString(&FilenameOrDirectory[i + 1]), ModDate, FileSize, false)));
			}
		}

		return true;
	}

	bool PassesFilterTest(const TCHAR* Filename)
	{		
		if (FilterList.Num() == 0)
		{
			return true; // no filters. everything passes.
		}

		FString BaseFile = Filename;
		FString Extension = FPaths::GetExtension(BaseFile, true);
		if (!Extension.IsEmpty() && FCString::IsNumeric(*Extension))
		{
			BaseFile.LeftChopInline(Extension.Len());
			const int32 DotPos = BaseFile.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (DotPos != INDEX_NONE)
			{
				Extension = BaseFile.Right(DotPos);
				Extension += TEXT(".*");
			}
		}

		// See if it matches any of the extensions
		for (const FString& FilterExt : FilterList)
		{
			if (FilterExt == TEXT("*") || FilterExt == TEXT(".*") || FilterExt == TEXT("*.*") || FilterExt.EndsWith(Extension))
			{
				return true;
			}
		}
		return false;
	}

private:
	TArray<TSharedPtr<FFileEntry>>& FileList;
	TArray<TSharedPtr<FFileEntry>>& FolderList;
	TArray<FString> FilterList;
};


class FSlateFileDialogDirVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	FSlateFileDialogDirVisitor(TArray<FString> *InDirectoryNames)
		: DirectoryNames(InDirectoryNames)
	{}

	void SetResultPath(TArray<FString> *InDirectoryNames) { DirectoryNames = InDirectoryNames; }

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		int32 i;

		// break filename from path
		for (i = FCString::Strlen(FilenameOrDirectory) - 1; i >= 0; i--)
		{
			if (FilenameOrDirectory[i] == TCHAR('/'))
				break;
		}

#if HIDE_HIDDEN_FILES
		if (FilenameOrDirectory[i + 1] == TCHAR('.'))
		{
			return true;
		}
#endif

		if (bIsDirectory)
		{
			DirectoryNames->Add(FString(&FilenameOrDirectory[i + 1]));
		}

		return true;
	}
	
private:
	TArray<FString> *DirectoryNames;
};


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
FSlateFileDlgWindow::FSlateFileDlgWindow(FSlateFileDialogsStyle *InStyleSet)
{
	StyleSet = InStyleSet;
}

namespace
{
	TSharedPtr<SWindow> FindMatchingWindowImpl(const TArray<TSharedRef<SWindow>>& RootWindows, const TFunctionRef<bool(const TSharedRef<SWindow>&)> Predicate)
	{
		for (const TSharedRef<SWindow>& Window : RootWindows)
		{
			if (Predicate(Window))
			{
				return Window;
			}

			if (TSharedPtr<SWindow> ChildWindow = FindMatchingWindowImpl(Window->GetChildWindows(), Predicate))
			{
				return ChildWindow;
			}
		}
		return nullptr;
	}

	TSharedPtr<SWindow> FindMatchingWindow(const TFunctionRef<bool(const TSharedRef<SWindow>&)> Predicate)
	{
		return FindMatchingWindowImpl(FSlateApplication::Get().GetTopLevelWindows(), Predicate);
	}

	TSharedPtr<const SWidget> GetParentWindowWidget(const void* ParentWindowHandle)
	{
		if (ParentWindowHandle)
		{
			TSharedPtr<SWindow> ParentWindow = FindMatchingWindow([ParentWindowHandle](const TSharedRef<SWindow>& Window)
			{
				const TSharedPtr<FGenericWindow> NativeWindow = Window->GetNativeWindow();
				return NativeWindow && NativeWindow->GetOSWindowHandle() == ParentWindowHandle;
			});

			if (ParentWindow)
			{
				return ParentWindow;
			}
		}
		return FGlobalTabmanager::Get()->GetRootWindow();
	}
}

bool FSlateFileDlgWindow::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex)
{
	FString StartDirectory = DefaultPath;
	TrimFilenameFromPath(StartDirectory);

	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.Title(LOCTEXT("SlateFileDialogsOpenFile","Open File"))
		.CreateTitleBar(true)
		.MinHeight(400.0f)
		.MinWidth(600.0f)
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.ClientSize(FVector2D(800, 500));

	DialogWidget = SNew(SSlateFileOpenDlg)
		.bMultiSelectEnabled(Flags == 1)
		.ParentWindow(ModalWindow)
		.CurrentPath(StartDirectory)
		.Filters(FileTypes)
		.WindowTitleText(DialogTitle)
		.StyleSet(StyleSet);

	DialogWidget->SetOutNames(&OutFilenames);
	DialogWidget->SetOutFilterIndex(&OutFilterIndex);
	DialogWidget->SetDefaultFilterIndex(DefaultFilterIndex);

	ModalWindow->SetContent( DialogWidget.ToSharedRef() );

	FSlateApplication::Get().AddModalWindow(ModalWindow, GetParentWindowWidget(ParentWindowHandle));

	return (DialogWidget->GetResponse() == EResult::Accept && OutFilenames.Num() > 0);
}


bool FSlateFileDlgWindow::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
	const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	int32 DummyIndex;
	return OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyIndex);
}


bool FSlateFileDlgWindow::OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
	FString& OutFoldername)
{
	int32 DummyIndex;
	TArray<FString> TempOut;
	FString Filters = "";

	FString StartDirectory = DefaultPath;
	TrimFilenameFromPath(StartDirectory);

	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.Title(LOCTEXT("SlateFileDialogsOpenDirectory","Open Directory"))
		.CreateTitleBar(true)
		.MinHeight(400.0f)
		.MinWidth(600.0f)
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.ClientSize(FVector2D(800, 500));

	DialogWidget = SNew(SSlateFileOpenDlg)
		.bMultiSelectEnabled(false)
		.ParentWindow(ModalWindow)
		.bDirectoriesOnly(true)
		.CurrentPath(StartDirectory)
		.WindowTitleText(DialogTitle)
		.StyleSet(StyleSet);

	DialogWidget->SetOutNames(&TempOut);
	DialogWidget->SetOutFilterIndex(&DummyIndex);

	ModalWindow->SetContent( DialogWidget.ToSharedRef() );

	FSlateApplication::Get().AddModalWindow(ModalWindow, GetParentWindowWidget(ParentWindowHandle));

	bool RC = (DialogWidget->GetResponse() == EResult::Accept && TempOut.Num() > 0);

	if (TempOut.Num() > 0)
	{
		OutFoldername = FPaths::ConvertRelativePathToFull(TempOut[0]);
		if (!OutFoldername.EndsWith(TEXT("/")))
		{
			OutFoldername += TEXT("/");
		}
	}

	return RC;
}


bool FSlateFileDlgWindow::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
	const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
	int32 DefaultFilterIndex)
{
	FString StartDirectory = DefaultPath;
	TrimFilenameFromPath(StartDirectory);

	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.Title(LOCTEXT("SlateFileDialogsSaveFile","Save File"))
		.CreateTitleBar(true)
		.MinHeight(400.0f)
		.MinWidth(600.0f)
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.ClientSize(FVector2D(800, 500));

	DialogWidget = SNew(SSlateFileOpenDlg)
		.bMultiSelectEnabled(false)
		.ParentWindow(ModalWindow)
		.bSaveFile(true)
		.AcceptText(LOCTEXT("SlateFileDialogsSave","Save"))
		.CurrentPath(StartDirectory)
		.Filters(FileTypes)
		.WindowTitleText(DialogTitle)
		.StyleSet(StyleSet);

	DialogWidget->SetOutNames(&OutFilenames);
	DialogWidget->SetOutFilterIndex(&OutFilterIndex);
	DialogWidget->SetDefaultFilterIndex(DefaultFilterIndex);
	DialogWidget->SetDefaultFile(DefaultFile);

	ModalWindow->SetContent( DialogWidget.ToSharedRef() );

	FSlateApplication::Get().AddModalWindow(ModalWindow, GetParentWindowWidget(ParentWindowHandle));

	return (DialogWidget->GetResponse() == EResult::Accept && OutFilenames.Num() > 0);
}

bool FSlateFileDlgWindow::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
	const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	int32 DummyIndex;
	return SaveFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyIndex);
}

void FSlateFileDlgWindow::TrimFilenameFromPath(FString &InPath)
{
	if (InPath.Len() == 0 || !FPaths::FileExists(InPath))
	{
		// No path given OR no file portion to trim
		return;
	}

	FPaths::CollapseRelativeDirectories(InPath);

	FString PathPart;
	FString FileNamePart;
	FString ExtensionPart;

	FPaths::Split(InPath, PathPart, FileNamePart, ExtensionPart);
	InPath = PathPart;
}

//-----------------------------------------------------------------------------
// custom file dialog widget
//-----------------------------------------------------------------------------
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSlateFileOpenDlg::Construct(const FArguments& InArgs)
{
	CurrentPath = InArgs._CurrentPath;
	Filters = InArgs._Filters;
	bMultiSelectEnabled = InArgs._bMultiSelectEnabled;
	bDirectoriesOnly = InArgs._bDirectoriesOnly;
	bSaveFile = InArgs._bSaveFile;
	WindowTitleText = InArgs._WindowTitleText;		
	OutNames = InArgs._OutNames;
	OutFilterIndex = InArgs._OutFilterIndex;
	UserResponse = FSlateFileDlgWindow::Cancel;
	ParentWindow = InArgs._ParentWindow;
	StyleSet = InArgs._StyleSet;
	AcceptText = InArgs._AcceptText;
	DirNodeIndex = -1;
	FilterIndex = 0;

	ESelectionMode::Type SelectMode = bMultiSelectEnabled ? ESelectionMode::Multi : ESelectionMode::Single;
	struct EVisibility SaveFilenameVisibility = bDirectoriesOnly ? EVisibility::Collapsed : EVisibility::Visible;

	this->ChildSlot
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(20.0f,20.0f))
			.BorderImage(StyleSet->GetBrush("SlateFileDialogs.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot() // window title
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 20.0f))
				[
					SAssignNew(WindowTitle, STextBlock)
					.Text(FText::FromString(WindowTitleText))
					.Font(StyleSet->GetFontStyle("SlateFileDialogs.DialogLarge"))
					.Justification(ETextJustify::Center)
				]

				+ SVerticalBox::Slot() // Path breadcrumbs
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				[
					SAssignNew(PathBreadcrumbTrail, SBreadcrumbTrail<FString>)
					.ButtonContentPadding(FMargin(2.0f, 2.0f))
					.ButtonStyle(StyleSet->Get(), "SlateFileDialogs.FlatButton")
					.DelimiterImage(StyleSet->GetBrush("SlateFileDialogs.PathDelimiter"))
					.TextStyle(StyleSet->Get(), "SlateFileDialogs.PathText")
					.ShowLeadingDelimiter(false)
					.OnCrumbClicked(this, &SSlateFileOpenDlg::OnPathClicked)
					.GetCrumbMenuContent(this, &SSlateFileOpenDlg::OnGetCrumbDelimiterContent)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserPath")))
				]

				+ SVerticalBox::Slot() // new directory
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					.AutoWidth()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SSlateFileOpenDlg::OnGoBackClick)
						.ContentPadding(FMargin(0.0f))
						[
							SNew(SImage)
							.Image(StyleSet->GetBrush("SlateFileDialogs.BrowseBack24"))
						]
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.0f, 0.0f, 40.0f, 0.0f))
					.AutoWidth()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SSlateFileOpenDlg::OnGoForwardClick)
						.ContentPadding(FMargin(0.0f))
						[
							SNew(SImage)
							.Image(StyleSet->GetBrush("SlateFileDialogs.BrowseForward24"))
						]
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.0f))
					.AutoWidth()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SSlateFileOpenDlg::OnNewDirectoryClick)
						.ContentPadding(FMargin(0.0f))
						[
							SNew(SImage)
							.Image(StyleSet->GetBrush("SlateFileDialogs.NewFolder24"))
						]
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(20.0f, 0.0f, 0.0f, 0.0f))
					.AutoWidth()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(NewDirectorySizeBox, SBox)
						.Padding(FMargin(0.0f))
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.WidthOverride(300.0f)
						.Visibility(EVisibility::Hidden)
						[
							SNew(SBorder)
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.Padding(FMargin(5.0f,0.0f))
							.BorderBackgroundColor(FLinearColor(0.1f, 0.1f, 0.1f, 1.0f))
							.BorderImage(StyleSet->GetBrush("SlateFileDialogs.WhiteBackground"))
							[
								SAssignNew(NewDirectoryEditBox, SInlineEditableTextBlock)
								.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
								.IsReadOnly(false)
								.Text(FText::GetEmpty())
								.OnTextCommitted(this, &SSlateFileOpenDlg::OnNewDirectoryCommitted)
								.OnVerifyTextChanged(this, &SSlateFileOpenDlg::OnNewDirectoryTextChanged)
							]
						]
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(FMargin(20.0f, 0.0f, 0.0f, 0.0f))
					.AutoWidth()
					[
						SAssignNew(NewDirCancelButton, SButton)
						.ContentPadding(FMargin(5.0f, 5.0f))
						.OnClicked(this, &SSlateFileOpenDlg::OnNewDirectoryAcceptCancelClick, FSlateFileDlgWindow::Cancel)
						.Text(LOCTEXT("SlateFileDialogsCancel","Cancel"))
						.Visibility(EVisibility::Hidden)
					]					
				]

				+ SVerticalBox::Slot() // new directory
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
				[
					SAssignNew(DirErrorMsg, STextBlock)
					.Font(StyleSet->GetFontStyle("SlateFileDialogs.DialogBold"))
					.Justification(ETextJustify::Left)
					.ColorAndOpacity(FLinearColor::Yellow)
					.Text(LOCTEXT("SlateFileDialogsDirError", "Unable to create directory!"))
					.Visibility(EVisibility::Collapsed)
				]

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillHeight(1.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.0f))
					.AutoWidth()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.Padding(FMargin(10.0f))
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.OnClicked(this, &SSlateFileOpenDlg::OnQuickLinkClick, FSlateFileDlgWindow::Project)
							.ContentPadding(FMargin(2.0f))
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(StyleSet->GetBrush("SlateFileDialogs.Folder24"))
								]

								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.AutoWidth()				
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ProjectsLabel", "Projects"))
									.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
									.Justification(ETextJustify::Left)
								]
							]
						]

						+ SVerticalBox::Slot()
						.Padding(FMargin(10.0f))
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.OnClicked(this, &SSlateFileOpenDlg::OnQuickLinkClick, FSlateFileDlgWindow::Engine)
							.ContentPadding(FMargin(2.0f))
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(SImage)
									.Image(StyleSet->GetBrush("SlateFileDialogs.Folder24"))
								]

								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.AutoWidth()				
								[
									SNew(STextBlock)
									.Text(LOCTEXT("EngineLabel", "Engine"))
									.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
									.Justification(ETextJustify::Left)
								]
							]
						]
					]

					+ SHorizontalBox::Slot() // spacer
					.Padding(FMargin(0.0f))
					.AutoWidth()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SSpacer)
						.Size(FVector2D(20.0f, 1.0f))
					]

					+ SHorizontalBox::Slot() // file list area
					.Padding(FMargin(0.0f, 0.0f, 20.0f, 0.0f))
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.FillWidth(1.0f)
					[
						SNew(SBorder)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.Padding(FMargin(10.0f))
						.BorderBackgroundColor(FLinearColor(0.10f, 0.10f, 0.10f, 1.0f))
						.BorderImage(StyleSet->GetBrush("SlateFileDialogs.WhiteBackground"))
						[
							SAssignNew(ListView, SListView<TSharedPtr<FFileEntry>>) // file list scroll
							.ListItemsSource(&LineItemArray)
							.SelectionMode(SelectMode)
							.OnGenerateRow(this, &SSlateFileOpenDlg::OnGenerateWidgetForList)
							.OnMouseButtonDoubleClick(this, &SSlateFileOpenDlg::OnItemDoubleClicked)
							.OnSelectionChanged(this, &SSlateFileOpenDlg::OnItemSelected)
							.HeaderRow
							(
								SNew(SHeaderRow)
								.Visibility(EVisibility::Visible)

								+ SHeaderRow::Column("Pathname")
									.DefaultLabel(LOCTEXT("SlateFileDialogsNameHeader", "Name"))
									.FillWidth(1.0f)

								+ SHeaderRow::Column("ModDate")
									.DefaultLabel(LOCTEXT("SlateFileDialogsModDateHeader", "Date Modified"))
									.FixedWidth(170.0f)

								+ SHeaderRow::Column("FileSize")
									.DefaultLabel(LOCTEXT("SlateFileDialogsFileSizeHeader", "File Size"))
									.FixedWidth(70.0f)
							)
						]
					]
				]

				+ SVerticalBox::Slot() // save filename entry
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(FMargin(0.0f, 10.0f, 50.0f, 0.0f))
				.AutoHeight()
				[
					SAssignNew(SaveFilenameSizeBox, SBox)
					.Padding(FMargin(0.0f))
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.MinDesiredHeight(20.0f)
					.Visibility(SaveFilenameVisibility)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(FMargin(0.0f))
						.AutoWidth()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FilenameLabel", "Filename:"))
							.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
							.Justification(ETextJustify::Left)
						]

						+ SHorizontalBox::Slot()
						.Padding(FMargin(10.0f, 0.0f, 0.0f, 0.0f))
						.AutoWidth()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(SBox)
							.Padding(FMargin(0.0f))
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.WidthOverride(300.0f)
							[
								SNew(SBorder)
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								.Padding(FMargin(5.0f,0.0f))
								.BorderBackgroundColor(FLinearColor(0.1f, 0.1f, 0.1f, 1.0f))
								.BorderImage(StyleSet->GetBrush("SlateFileDialogs.WhiteBackground"))
								[
									SAssignNew(SaveFilenameEditBox, SInlineEditableTextBlock)
									.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
									.IsReadOnly(false)
									.Text(FText::GetEmpty())
									.OnTextCommitted(this, &SSlateFileOpenDlg::OnFileNameCommitted)
								]
							]
						]
					]
				]

				+ SVerticalBox::Slot()  // cancel:accept buttons
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.0f))
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					[
						SAssignNew(FilterHBox, SHorizontalBox)

						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Bottom)
						.AutoWidth()
						.Padding(FMargin(0.0f))
						[
							SNew(STextBlock)
							.Text(bSaveFile ? LOCTEXT("SaveTypeLabel", "Save as type:") : LOCTEXT("FilterLabel", "Filter:"))
							.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
							.Justification(ETextJustify::Left)
						]

						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.AutoWidth()
						.Padding(FMargin(10.0f, 0.0f, 0.0f, 0.0f))
						[
							SNew(SBox)
							.MinDesiredWidth(200.0f)
							.MaxDesiredWidth(200.0f)
							.Padding(FMargin(0.0f))
							[
								SAssignNew(FilterCombo, SComboBox<TSharedPtr<FString>>)
								.ContentPadding(FMargin(4.0f, 2.0f))
								.OptionsSource(&FilterNameArray)
								.OnGenerateWidget_Lambda([this](TSharedPtr<FString> Item)
								{ 
									return SNew(SBox)
										.MaxDesiredWidth(600.0f)
										[
											SNew(STextBlock)
											.Text(FText::FromString(*Item))
											.Font(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"))
										];
								} )	
								.OnSelectionChanged(this, &SSlateFileOpenDlg::OnFilterChanged)
								[
									SAssignNew(FilterComboBoxTitleBlock, STextBlock)
								]
							]
						]
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.FillWidth(1.0f)
					[
						SNew(SSpacer)
							.Size(FVector2D(1.0f, 1.0f))
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(FMargin(0.0f, 0.0f, 20.0f, 0.0f))
					.AutoWidth()
					[
						SNew(SButton)
						.ContentPadding(FMargin(5.0f, 5.0f))
						.IsEnabled(this, &SSlateFileOpenDlg::IsAcceptEnabled)
						.OnClicked(this, &SSlateFileOpenDlg::OnAcceptCancelClick, FSlateFileDlgWindow::Accept)
						.Text(AcceptText)
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(FMargin(0.0f))
					.AutoWidth()
					[
						SNew(SButton)
						.ContentPadding(FMargin(5.0f, 5.0f))
						.OnClicked(this, &SSlateFileOpenDlg::OnAcceptCancelClick, FSlateFileDlgWindow::Cancel)
						.Text(LOCTEXT("SlateFileDialogsCancel","Cancel"))
					]
				]
			]
		];

	SaveFilename = "";

	bNeedsBuilding = true;
	bRebuildDirPath = true;
	bDirectoryHasChanged = false;
	DirectoryWatcher = nullptr;

	if (CurrentPath.Len() > 0 && !CurrentPath.EndsWith("/"))
	{
		CurrentPath = CurrentPath + TEXT("/");
	}

	HistoryIndex = 0;
	History.Add(CurrentPath);

#if ENABLE_DIRECTORY_WATCHER	
	if (!FModuleManager::Get().IsModuleLoaded("DirectoryWatcher"))
	{
		FModuleManager::Get().LoadModule("DirectoryWatcher");
	}

	FDirectoryWatcherModule &DirWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	DirectoryWatcher = DirWatcherModule.Get();
#endif
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SSlateFileOpenDlg::~SSlateFileOpenDlg()
{
	if (DirectoryWatcher && RegisteredPath.Len() > 0)
	{
		DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(RegisteredPath, OnDialogDirectoryChangedDelegateHandle);
	}
}

void SSlateFileOpenDlg::BuildDirectoryPath()
{
	// Clean up path as needed. Fix slashes and convert to absolute path.
	FString NormPath = CurrentPath;
	FPaths::NormalizeFilename(NormPath);
	FPaths::RemoveDuplicateSlashes(NormPath);
	FString AbsPath = FPaths::ConvertRelativePathToFull(NormPath);
	TCHAR Temp[MAX_PATH_LENGTH];

	DirectoryNodesArray.Empty();

	FString BuiltPath;
	if (PLATFORM_WINDOWS)
	{
		int32 Idx;

		if (AbsPath.FindChar(TCHAR('/'), Idx))
		{
			BuiltPath = BuiltPath + TEXT("/") + AbsPath.Left(Idx);
		}

		FCString::Strncpy(Temp, &AbsPath[Idx < AbsPath.Len() - 1 ? Idx + 1 : Idx], UE_ARRAY_COUNT(Temp));

		DirectoryNodesArray.Add(FDirNode(AbsPath.Left(Idx), nullptr));
	}
	else if (PLATFORM_LINUX)
	{
		// start with system base directory
		FCString::Strncpy(Temp, *AbsPath, UE_ARRAY_COUNT(Temp));

		BuiltPath = "/";
		DirectoryNodesArray.Add(FDirNode(FString(TEXT("/")), nullptr));
	}
	else
	{
		checkf(false, TEXT("SlateDialogs will not work on this platform (modify SSlateFileOpenDlg::BuildDirectoryPath())"));
	}

	// break path into tokens
	TCHAR *ContextStr = nullptr;
	TCHAR *DirNode = FCString::Strtok(Temp, TEXT("/"), &ContextStr);

	while (DirNode)
	{
		FString Label = DirNode;
		DirectoryNodesArray.Add(FDirNode(Label, nullptr));
		BuiltPath = BuiltPath + Label + TEXT("/");
	
		DirNode = FCString::Strtok(nullptr, TEXT("/"), &ContextStr);
	}
	
	RefreshCrumbs();
}


void SSlateFileOpenDlg::RefreshCrumbs()
{
	// refresh crumb list
	if (PathBreadcrumbTrail.IsValid())
	{
		PathBreadcrumbTrail->ClearCrumbs();

		FString BuiltPath;
		if (PLATFORM_WINDOWS)
		{
			PathBreadcrumbTrail->PushCrumb(LOCTEXT("SlateFileDialogsSystem", "System"), FString("SYSTEM"));

			for (int32 i = 0; i < DirectoryNodesArray.Num(); i++)
			{
				BuiltPath = BuiltPath + DirectoryNodesArray[i].Label + TEXT("/");
				PathBreadcrumbTrail->PushCrumb(FText::FromString(DirectoryNodesArray[i].Label), BuiltPath);
			}
		}
		else if (PLATFORM_LINUX)
		{
			BuiltPath = "/";
			PathBreadcrumbTrail->PushCrumb(FText::FromString(BuiltPath), BuiltPath);

			for (int32 i = 1; i < DirectoryNodesArray.Num(); i++)
			{
				BuiltPath = BuiltPath + DirectoryNodesArray[i].Label + TEXT("/");
				PathBreadcrumbTrail->PushCrumb(FText::FromString(DirectoryNodesArray[i].Label), BuiltPath);
			}
		}
	}
}


void SSlateFileOpenDlg::OnPathClicked(const FString& NewPath)
{
	if (NewPath.Compare("SYSTEM") == 0)
	{
		// Ignore clicks on the virtual root. (Only happens for Windows systems.)
		return;
	}

	// set new current path and flag that we need to update directory display
	CurrentPath = NewPath;
	bRebuildDirPath = true;
	bNeedsBuilding = true;

	if ((History.Num()-HistoryIndex-1) > 0)
	{
		History.RemoveAt(HistoryIndex+1, History.Num()-HistoryIndex-1, EAllowShrinking::Yes);
	}

	History.Add(CurrentPath);
	HistoryIndex++;

	RefreshCrumbs();
}


void SSlateFileOpenDlg::OnPathMenuItemClicked( FString ClickedPath )
{	
	CurrentPath = ClickedPath;
	bRebuildDirPath = true;
	bNeedsBuilding = true;

	if ((History.Num()-HistoryIndex-1) > 0)
	{
		History.RemoveAt(HistoryIndex+1, History.Num()-HistoryIndex-1, EAllowShrinking::Yes);
	}

	History.Add(CurrentPath);
	HistoryIndex++;
	
	RefreshCrumbs();
}


TSharedRef<SWidget> SSlateFileOpenDlg::OnGetCrumbDelimiterContent(const FString& CrumbData) const
{
	TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
	TArray<FString> SubDirs;

	IFileManager& FileManager = IFileManager::Get();	
	FSlateFileDialogDirVisitor DirVisitor(&SubDirs);

	if (PLATFORM_WINDOWS)
	{
		if (CrumbData.Compare("SYSTEM") == 0)
		{
			// Windows doesn't have a root file system. So we need to provide a way to select system drives.
			// This is done by creating a virtual root using 'System' as the top node.
			int32 DrivesMask =
#if PLATFORM_WINDOWS
					(int32)GetLogicalDrives()
#else
					0
#endif // PLATFORM_WINDOWS
					;

			FMenuBuilder MenuBuilder(true, NULL);
			const TCHAR *DriveLetters = TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ");
			FString Drive = TEXT("A:");

			for (int32 i = 0; i < 26; i++)
			{
				if (DrivesMask & 0x01)
				{
					Drive[0] = DriveLetters[i];

					MenuBuilder.AddMenuEntry(
						FText::FromString(Drive),
						FText::GetEmpty(),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateSP(const_cast<SSlateFileOpenDlg*>(this), &SSlateFileOpenDlg::OnPathMenuItemClicked, Drive + TEXT("/"))));
				}

				DrivesMask >>= 1;
			}

			return SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.MaxHeight(400.0f)
				[
					MenuBuilder.MakeWidget()
				];
		}
	}

	FileManager.IterateDirectory(*CrumbData, DirVisitor);

	if (SubDirs.Num() > 0)
	{
		SubDirs.Sort();

		FMenuBuilder MenuBuilder(  true, NULL );

		for (int32 i = 0; i < SubDirs.Num(); i++)
		{
			const FString& SubDir = SubDirs[i];

			MenuBuilder.AddMenuEntry(
				FText::FromString(SubDir),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(const_cast<SSlateFileOpenDlg*>(this), &SSlateFileOpenDlg::OnPathMenuItemClicked, CrumbData + SubDir + TEXT("/"))));
		}

		Widget = 
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.MaxHeight(400.0f)
			[
				MenuBuilder.MakeWidget()
			];
	}

	return Widget.ToSharedRef();
}


FReply SSlateFileOpenDlg::OnQuickLinkClick(FSlateFileDlgWindow::EResult ButtonID)
{
	if (ButtonID == FSlateFileDlgWindow::Project)
	{
		// Taken from DesktopPlatform. We have to do this to avoid a circular dependency.
		const FString DefaultProjectSubFolder =TEXT("Unreal Projects");
		CurrentPath = FPaths::ConvertRelativePathToFull(FString(FPlatformProcess::UserDir()) + DefaultProjectSubFolder + TEXT("/"));
	}

	if (ButtonID == FSlateFileDlgWindow::Engine)
	{
		CurrentPath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());
	}
	
	if ((History.Num()-HistoryIndex-1) > 0)
	{
		History.RemoveAt(HistoryIndex+1, History.Num()-HistoryIndex-1, EAllowShrinking::Yes);
	}

	History.Add(CurrentPath);
	HistoryIndex++;

	bNeedsBuilding = true;
	bRebuildDirPath = true;

	return FReply::Handled();
}

void SSlateFileOpenDlg::SetOutputFiles()
{
	if (OutNames != nullptr)
	{
		TArray<FString> NamesArray;
		ParseTextField(NamesArray, SaveFilename);

		OutNames->Empty();

		if (bDirectoriesOnly)
		{
			if (NamesArray.Num() > 0)
			{
				FString Path = CurrentPath + NamesArray[0];
				OutNames->Add(Path);
			}
			else
			{
				// select the current directory
				OutNames->Add(CurrentPath);
			}
		}
		else
		{
			for (int32 i=0; i < NamesArray.Num(); i++)
			{
				FString Path = CurrentPath + NamesArray[i];
				OutNames->Add(Path);
			}

			if (OutFilterIndex != nullptr)
			{
				*(OutFilterIndex) = FilterIndex;
			}
		}
	}
}

TSharedPtr<FFileEntry> SSlateFileOpenDlg::GetSoloDirectorySelected() const
{
	TArray<TSharedPtr<FFileEntry>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() == 1 && SelectedItems[0]->bIsDirectory)
	{
		return SelectedItems[0];
	}

	return nullptr;
}

bool SSlateFileOpenDlg::IsAcceptEnabled() const
{
	if (bDirectoriesOnly)
	{
		return true;
	}

	TSharedPtr<FFileEntry> SoloSelectedDirectory = GetSoloDirectorySelected();

	if (SoloSelectedDirectory.IsValid()) 
	{
		return true;
	}
	else if (!SaveFilename.IsEmpty())
	{
		return true;
	}

	return false;
}

FReply SSlateFileOpenDlg::OnAcceptCancelClick(FSlateFileDlgWindow::EResult ButtonID)
{
	if (ButtonID == FSlateFileDlgWindow::Accept)
	{
		if (!bDirectoriesOnly)
		{
			TSharedPtr<FFileEntry> SoloSelectedDirectory = GetSoloDirectorySelected();

			if (SoloSelectedDirectory.IsValid())
			{
				OnItemDoubleClicked(SoloSelectedDirectory);
				return FReply::Handled();
			}
		}

		SetOutputFiles();
	}
	else
	{
		if (OutNames != nullptr)
		{
			OutNames->Empty();
		}
	}

	UserResponse = ButtonID;
	ParentWindow.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SSlateFileOpenDlg::OnDirSublevelClick(int32 Level)
{
	DirectoryNodesArray[DirNodeIndex].TextBlock->SetFont(StyleSet->GetFontStyle("SlateFileDialogs.Dialog"));

	FString NewPath = TEXT("/");

	for (int32 i = 1; i <= Level; i++)
	{
		NewPath += DirectoryNodesArray[i].Label + TEXT("/");
	}

	CurrentPath = NewPath;
	bRebuildDirPath = false;
	bNeedsBuilding = true;

	DirNodeIndex = Level;
	DirectoryNodesArray[DirNodeIndex].TextBlock->SetFont(StyleSet->GetFontStyle("SlateFileDialogs.DialogBold"));

	return FReply::Handled();
}

void SSlateFileOpenDlg::Tick(const FGeometry &AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (DirectoryWatcher)
	{
		DirectoryWatcher->Tick(InDeltaTime);
	}

	if (bDirectoryHasChanged && !bNeedsBuilding)
	{
		ReadDir(true);
		RebuildFileTable();
		ListView->RequestListRefresh();
		bDirectoryHasChanged = false;			
	}		

	if (bNeedsBuilding)
	{
		// quick-link buttons to directory sublevels
		if (bRebuildDirPath)
		{
			BuildDirectoryPath();
		}

		// Get directory contents and rebuild list
		ParseFilters();
		ReadDir();
		RebuildFileTable();
		ListView->RequestListRefresh();
	}

	bNeedsBuilding = false;
	bRebuildDirPath = false;
}


void SSlateFileOpenDlg::ReadDir(bool bIsRefresh)
{
	if (DirectoryWatcher && RegisteredPath.Len() > 0 && !bIsRefresh)
	{
		DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(RegisteredPath, OnDialogDirectoryChangedDelegateHandle);
		RegisteredPath = TEXT("");
	}
	
	IFileManager& FileManager = IFileManager::Get();

	FilesArray.Empty();
	FoldersArray.Empty();
	FString FilterList;

	if (FilterListArray.Num() > 0 && FilterIndex >= 0)
	{
		FilterList = FilterListArray[FilterIndex];
	}

	FSlateFileDialogVisitor DirVisitor(FilesArray, FoldersArray, FilterList);

	FileManager.IterateDirectory(*CurrentPath, DirVisitor);
	
	FilesArray.Sort(FFileEntry::ConstPredicate);
	FoldersArray.Sort(FFileEntry::ConstPredicate);

	if (DirectoryWatcher && !bIsRefresh)
	{
		DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(CurrentPath,
						IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &SSlateFileOpenDlg::OnDirectoryChanged),
						OnDialogDirectoryChangedDelegateHandle, IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges | IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree);

		RegisteredPath = CurrentPath;
	}
}


void SSlateFileOpenDlg::OnDirectoryChanged(const TArray <FFileChangeData> &FileChanges)
{
	bDirectoryHasChanged = true;
}



void SSlateFileOpenDlg::RebuildFileTable()
{
	LineItemArray.Empty();
	
	// directory entries
	for (int32 i = 0; i < FoldersArray.Num(); i++)
	{
		LineItemArray.Add(FoldersArray[i]);
	}

	// file entries
	if (bDirectoriesOnly == false)
	{
		for (int32 i = 0; i < FilesArray.Num(); i++)
		{
			LineItemArray.Add(FilesArray[i]);
		}
	}
}


TSharedRef<ITableRow> SSlateFileOpenDlg::OnGenerateWidgetForList(TSharedPtr<FFileEntry> Item,
		const TSharedRef<STableViewBase> &OwnerTable)
{
	return SNew(SSlateFileDialogRow, OwnerTable)
			.DialogItem(Item)
			.StyleSet(StyleSet);
}


void SSlateFileOpenDlg::OnItemDoubleClicked(TSharedPtr<FFileEntry> Item)
{
	if (Item->bIsDirectory)
	{
		if (!bSaveFile)
		{
			SetDefaultFile(FString(""));
		}

		CurrentPath = CurrentPath + Item->Label + TEXT("/");
		bNeedsBuilding = true;
		bRebuildDirPath = true;

		if ((History.Num()-HistoryIndex-1) > 0)
		{
			History.RemoveAt(HistoryIndex+1, History.Num()-HistoryIndex-1, EAllowShrinking::Yes);
		}

		History.Add(CurrentPath);
		HistoryIndex++;
	}
	else
	{
		SetOutputFiles();
		UserResponse = FSlateFileDlgWindow::Accept;
		ParentWindow.Pin()->RequestDestroyWindow();
	}
}


void SSlateFileOpenDlg::OnFilterChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	for (int32 i = 0; i < FilterNameArray.Num(); i++)
	{
		if (NewValue.IsValid() && FilterNameArray[i].Get()->Compare(*NewValue.Get(), ESearchCase::CaseSensitive) == 0)
		{
			FilterIndex = i;
			break;
		}
	}

	bNeedsBuilding = true;
}

void SSlateFileOpenDlg::ParseTextField(TArray<FString> &FilenameArray, FString Files)
{
	FString FileList = Files;
	FileList.TrimStartAndEndInline();

	FilenameArray.Empty();

	if  (FileList.Len() > 0 && FileList[0] == TCHAR('"'))
	{
		FString TempName;
		SaveFilename.Empty();

		for (int32 i = 0; i < FileList.Len(); )
		{
			// find opening quote (")
			for (; i < FileList.Len() && FileList[i] != TCHAR('"'); i++);
			
			if (i >= FileList.Len())
			{
				break;
			}
			
			// copy name until closing quote is found.
			TempName.Empty();
			for (i++; i < FileList.Len() && FileList[i] != TCHAR('"'); i++)
			{
				TempName.AppendChar(FileList[i]);
			}

			if (i >= FileList.Len())
			{
				break;
			}
			
			// check to see if file exists or if we're trying to save a file. if so, add it to list.
			if (FPaths::FileExists(CurrentPath + TempName) || bSaveFile)
			{
				FilenameArray.Add(TempName);
			}

			// if multiselect is off, don't bother parsing out any additional file names.
			if (!bMultiSelectEnabled)
			{
				break;
			}

			i++;
		}
	}
	else
	{
		TArray<FString> Extensions;
		FString FirstNonWilcardExtension;

		// get current filter extension
		if (!bDirectoriesOnly && GetFilterExtensions(Extensions))
		{
			bool bSaveFilenameHasExtension = false;

			for ( const FString& Extension : Extensions )
			{
				bSaveFilenameHasExtension = ( !IsWildcardExtension(Extension) && SaveFilename.EndsWith(Extension, ESearchCase::CaseSensitive) );

				if ( !IsWildcardExtension(Extension) && FirstNonWilcardExtension.IsEmpty() )
				{
					FirstNonWilcardExtension = Extension;
				}

				if ( bSaveFilenameHasExtension )
				{
					break;
				}
			}

			// append extension to filename if user left it off
			if (!bSaveFilenameHasExtension && !FirstNonWilcardExtension.IsEmpty())
			{
				Files = Files + FirstNonWilcardExtension;
			}
		}

		FilenameArray.Add(Files);
	}
}

void SSlateFileOpenDlg::SetDefaultFile(FString DefaultFile)
{
	FString FileList = DefaultFile;
	FileList.TrimStartAndEndInline();

	if  (FileList.Len() > 0 && FileList[0] == TCHAR('"'))
	{
		TArray<FString> NamesArray;
		ParseTextField(NamesArray, FileList);

		SaveFilename.Empty();

		for (int32 i = 0; i < NamesArray.Num(); i++)
		{	
			SaveFilename = SaveFilename + TEXT("\"") + NamesArray[i] + TEXT("\" ");

			// if multiselect is off, don't bother adding any additional file names.
			if (!bMultiSelectEnabled)
			{
				break;
			}
		}
	}
	else
	{
		SaveFilename = FileList;
	}

	SaveFilenameEditBox->SetText(SaveFilename);
}


void SSlateFileOpenDlg::OnFileNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	// update edit box unless user choose to escape out
	if (InCommitType != ETextCommit::OnCleared)
	{
		SaveFilename = InText.ToString();
		ListView->ClearSelection();

		SetDefaultFile(SaveFilename);
	}
}


void SSlateFileOpenDlg::OnItemSelected(TSharedPtr<FFileEntry> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		FString FileList;

		if (!bDirectoriesOnly)
		{
			TArray<TSharedPtr<FFileEntry>> SelectedItems = ListView->GetSelectedItems();

			for (int32 i = 0; i < SelectedItems.Num(); i++)
			{
				if (!SelectedItems[i]->bIsDirectory)
				{
					FileList = FileList + TEXT("\"") + SelectedItems[i]->Label + TEXT("\" ");
				}
			}
		}
		else
		{
			FileList = Item->Label;
		}

		// Update file name text as long as we aren't saving a file with a directory selected
		if (!(bSaveFile && Item->bIsDirectory))
		{
			SetDefaultFile(FileList);
		}
	}
}


void SSlateFileOpenDlg::ParseFilters()
{
	if (FilterCombo.IsValid() && FilterHBox.IsValid())
	{
		if (Filters.Len() > 0)
		{
			if (FilterNameArray.Num() == 0)
			{
				TCHAR Temp[MAX_FILTER_LENGTH] = {0};
				FCString::Strncpy(Temp, *Filters, UE_ARRAY_COUNT(Temp));

				// break path into tokens
				TCHAR *ContextStr = nullptr;

				TCHAR *FilterDescription = FCString::Strtok(Temp, TEXT("|"), &ContextStr);
				TCHAR *FilterList;

				while (FilterDescription)
				{
					// filter wild cards
					FilterList = FCString::Strtok(nullptr, TEXT("|"), &ContextStr);

					FilterNameArray.Add(MakeShareable(new FString(FilterDescription)));
					FilterListArray.Add(FString(FilterList));

					// next filter entry
					FilterDescription = FCString::Strtok(nullptr, TEXT("|"), &ContextStr);
				}
			}

			FilterComboBoxTitleBlock->SetText(FText::FromString(*FilterNameArray[FilterIndex]));
		}
		else
		{
			FilterNameArray.Empty();
			FilterHBox->SetVisibility(EVisibility::Hidden);
		}
	}
}


bool SSlateFileOpenDlg::GetFilterExtensions(TArray<FString>& OutExtensions)
{
	OutExtensions.Reset();

	// check to see if filters were given
	if (Filters.Len() == 0)
	{
		return false;
	}

	// We have attempted to get the filter extension before parsing them
	if (FilterListArray.Num() == 0)
	{
		ParseFilters();
	}

	if (!FilterListArray.IsValidIndex(FilterIndex))
	{
		return false;
	}

	TArray<FString> Extensions;
	const bool bCullEmpty = true;
	if (FilterListArray[FilterIndex].ParseIntoArray(Extensions, TEXT(";"), bCullEmpty) > 0)
	{
		for (FString& Extension : Extensions)
		{
			// find start of extension
			int32 DotIndex;
			if (Extension.FindChar(TEXT('.'), DotIndex))
			{
				Extension.RightChopInline(DotIndex);

				// strip any trailing junk
				for (TCHAR& ExtensionChar : Extension)
				{
					if (ExtensionChar == TEXT(' ') || ExtensionChar == TEXT(')') || ExtensionChar == TEXT(';'))
					{
						ExtensionChar = 0;
						break;
					}
				}

				Extension.TrimToNullTerminator();

				// store result and clean up
				OutExtensions.Add(Extension);
			}
			else if (Extension[0] == TEXT('*'))
			{
				OutExtensions.Add(Extension);
			}
		}
	}

	return OutExtensions.Num() > 0;
}

bool SSlateFileOpenDlg::IsWildcardExtension(const FString& Extension)
{
	return (Extension.Find(TEXT(".*")) >= 0) ||
			(Extension.Find(TEXT("*")) >= 0);
}

void SSlateFileOpenDlg::OnNewDirectoryCommitted(const FText & InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		OnNewDirectoryAcceptCancelClick(FSlateFileDlgWindow::Accept);	
	}
	else
	{
		OnNewDirectoryAcceptCancelClick(FSlateFileDlgWindow::Cancel);
	}
}

void SSlateFileOpenDlg::OnNewDirectoryClickPostTick(float)
{
	if (FSlateApplication::Get().GetUserFocusedWidget(0) == NewDirectoryEditBox)
	{
		NewDirectoryEditBox->EnterEditingMode();

		// The post tick event handler is not required anymore.
		FSlateApplication::Get().OnPostTick().Remove(PostTickHandle);
		PostTickHandle.Reset();
	}
}

FReply SSlateFileOpenDlg::OnNewDirectoryClick()
{
	NewDirectorySizeBox->SetVisibility(EVisibility::Visible);
	NewDirCancelButton->SetVisibility(EVisibility::Visible);
	NewDirectoryEditBox->SetText(FString(""));
	NewDirectoryEditBox->SetVisibility(EVisibility::Visible);
	DirErrorMsg->SetVisibility(EVisibility::Collapsed);

	PostTickHandle = FSlateApplication::Get().OnPostTick().AddSP(this, &SSlateFileOpenDlg::OnNewDirectoryClickPostTick);
	
	return FReply::Handled().SetUserFocus(NewDirectoryEditBox.ToSharedRef(), EFocusCause::SetDirectly);
}


bool SSlateFileOpenDlg::OnNewDirectoryTextChanged(const FText &InText, FText &ErrorMsg)
{
	NewDirectoryName = InText.ToString();
	return true;
}


FReply SSlateFileOpenDlg::OnNewDirectoryAcceptCancelClick(FSlateFileDlgWindow::EResult ButtonID)
{
	if (ButtonID == FSlateFileDlgWindow::Accept)
	{
		NewDirectoryName.TrimStartAndEndInline();

		if (NewDirectoryName.Len() > 0)
		{
			IPlatformFile &PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			FString DirPath = CurrentPath + NewDirectoryName;

			if (!PlatformFile.CreateDirectory(*DirPath))
			{
				DirErrorMsg->SetVisibility(EVisibility::Visible);				
				return FReply::Handled();
			}

			bDirectoryHasChanged = true;
		}
	}

	NewDirectorySizeBox->SetVisibility(EVisibility::Hidden);
	NewDirCancelButton->SetVisibility(EVisibility::Hidden);
	DirErrorMsg->SetVisibility(EVisibility::Collapsed);
	NewDirectoryEditBox->SetVisibility(EVisibility::Hidden);
	NewDirectoryEditBox->SetText(FString(""));

	return FReply::Handled();
}


FReply SSlateFileOpenDlg::OnGoForwardClick()
{
	if ((HistoryIndex+1) < History.Num())
	{
		if (!bSaveFile)
		{
			SetDefaultFile(FString(""));
		}

		HistoryIndex++;
		CurrentPath = History[HistoryIndex];
		bNeedsBuilding = true;
		bRebuildDirPath = true;
		bDirectoryHasChanged = false;
	}

	return FReply::Handled();
}


FReply SSlateFileOpenDlg::OnGoBackClick()
{
	if (HistoryIndex > 0)
	{
		if (!bSaveFile)
		{
			SetDefaultFile(FString(""));
		}

		HistoryIndex--;
		CurrentPath = History[HistoryIndex];
		bNeedsBuilding = true;
		bRebuildDirPath = true;
		bDirectoryHasChanged = false;	
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
