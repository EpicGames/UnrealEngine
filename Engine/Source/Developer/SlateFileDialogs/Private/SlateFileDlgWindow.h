// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Paths.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SComboBox.h" 
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "SlateFileDialogsStyles.h"
#include "IDirectoryWatcher.h"

#define LOCTEXT_NAMESPACE "SlateFileDialogsNamespace"

class SButton;
class SSlateFileOpenDlg;
class SWindow;

//-----------------------------------------------------------------------------
struct FFileEntry
{
	FString	Label;
	FString ModDate;
	FString FileSize;
	bool bIsSelected;
	bool bIsDirectory;

	FFileEntry() { }

	FFileEntry(FString InLabel, FString InModDate, FString InFileSize, bool InIsDirectory)
		: Label(MoveTemp(InLabel))
		, ModDate(MoveTemp(InModDate))
		, FileSize(MoveTemp(InFileSize))
		, bIsSelected(false)
		, bIsDirectory(InIsDirectory)
	{
	}

	inline static bool ConstPredicate ( const TSharedPtr<FFileEntry> Entry1, const TSharedPtr<FFileEntry> Entry2)
	{
		return (Entry1->Label.Compare( Entry2->Label ) < 0);
	}
};

typedef TSharedPtr<FFileEntry> SSlateFileDialogItemPtr;


//-----------------------------------------------------------------------------
class FSlateFileDlgWindow
{
public:
	enum EResult
	{
		Cancel = 0,
		Accept = 1,
		Engine = 2,
		Project = 3
	};

	FSlateFileDlgWindow(FSlateFileDialogsStyle *InStyleSet);

	bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex = 0);

	bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames);

	bool OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		FString& OutFoldername);

	bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex = 0);

	bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames);

private:	
	TSharedPtr<class SSlateFileOpenDlg> DialogWidget;
	FString CurrentDirectory;

	FSlateFileDialogsStyle* StyleSet;

	void TrimFilenameFromPath(FString &Path);
};



//-----------------------------------------------------------------------------
class SSlateFileOpenDlg : public SCompoundWidget
{
public:
	struct FDirNode
	{
		FString	Label;
		TSharedPtr<STextBlock> TextBlock;
		TSharedPtr<SButton> Button;

		FDirNode() { }

		FDirNode(FString InLabel, TSharedPtr<STextBlock> InTextBlock)
		{
			Label = InLabel;
			TextBlock = InTextBlock;
		}
	};

	SLATE_BEGIN_ARGS(SSlateFileOpenDlg)
		: _CurrentPath(TEXT("")),
		_Filters(TEXT("")),
		_bMultiSelectEnabled(false),
		_WindowTitleText(TEXT("")),
		_AcceptText(LOCTEXT("SlateDialogOpen","Open")),
		_bDirectoriesOnly(false),
		_bSaveFile(false),
		_OutNames(nullptr),
		_OutFilterIndex(nullptr),
		_ParentWindow(nullptr),
		_StyleSet(nullptr)
	{}

	SLATE_ARGUMENT(FString, CurrentPath)
	SLATE_ARGUMENT(FString, Filters)
	SLATE_ARGUMENT(bool, bMultiSelectEnabled)
	SLATE_ARGUMENT(FString, WindowTitleText)
	SLATE_ARGUMENT(FText, AcceptText)
	SLATE_ARGUMENT(bool, bDirectoriesOnly)
	SLATE_ARGUMENT(bool, bSaveFile)
	SLATE_ARGUMENT(TArray<FString>*, OutNames)
	SLATE_ARGUMENT(int32*, OutFilterIndex)
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
	SLATE_ARGUMENT(FSlateFileDialogsStyle*, StyleSet)
	SLATE_END_ARGS()

	~SSlateFileOpenDlg();

	void Construct(const FArguments& InArgs);
	FSlateFileDlgWindow::EResult GetResponse() { return UserResponse; }
	void SetOutNames(TArray<FString>* Ptr) { OutNames = Ptr; }
	void SetOutFilterIndex(int32* InOutFilterIndex) { OutFilterIndex = InOutFilterIndex; }
	void SetDefaultFilterIndex(int32 DefaultFilterIndex) { FilterIndex = DefaultFilterIndex; }
	void SetDefaultFile(FString DefaultFile);

private:	
	void OnPathClicked( const FString& CrumbData );
	TSharedRef<SWidget> OnGetCrumbDelimiterContent(const FString& CrumbData) const;
	void RebuildFileTable();
	void BuildDirectoryPath();
	void ReadDir(bool bIsRefresh = false);
	void RefreshCrumbs();
	void OnPathMenuItemClicked( FString ClickedPath );
	void OnItemSelected(TSharedPtr<FFileEntry> Item, ESelectInfo::Type SelectInfo);
	void ParseTextField(TArray<FString> &FilenameArray, FString Files);
	void Tick(const FGeometry &AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FFileEntry> Item, const TSharedRef<STableViewBase> &OwnerTable);
	void OnItemDoubleClicked(TSharedPtr<FFileEntry> Item);

	void OnFilterChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	bool IsAcceptEnabled() const;
	FReply OnAcceptCancelClick(FSlateFileDlgWindow::EResult ButtonID);
	FReply OnQuickLinkClick(FSlateFileDlgWindow::EResult ButtonID);
	FReply OnDirSublevelClick(int32 Level);
	void OnDirectoryChanged(const TArray <FFileChangeData> &FileChanges);	
	void OnFileNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	void OnNewDirectoryCommitted(const FText& InText, ETextCommit::Type InCommitType);
	FReply OnNewDirectoryClick();
	bool OnNewDirectoryTextChanged(const FText &InText, FText &ErrorMsg);
	FReply OnNewDirectoryAcceptCancelClick(FSlateFileDlgWindow::EResult ButtonID);

	void OnNewDirectoryClickPostTick(float);

	FReply OnGoForwardClick();
	FReply OnGoBackClick();

	/** Collects the output files. */
	void SetOutputFiles();

	bool GetFilterExtensions(TArray< FString >& OutExtensions);
	void ParseFilters();
	/** @return true if the extension filter contains a wildcard or not */
	bool IsWildcardExtension(const FString& Extension);

	/** @return ptr to directory if only one directory and zero files are selected, else nullptr */
	TSharedPtr<FFileEntry> GetSoloDirectorySelected() const;

	TArray< FDirNode > DirectoryNodesArray;
	TArray<TSharedPtr<FFileEntry>> FoldersArray;
	TArray<TSharedPtr<FFileEntry>> FilesArray;
	TArray<TSharedPtr<FFileEntry>> LineItemArray;	

	TSharedPtr<SComboBox<TSharedPtr<FString>>> FilterCombo;
	TSharedPtr<STextBlock> FilterComboBoxTitleBlock;
	TSharedPtr<SHorizontalBox> FilterHBox;
	TSharedPtr<SInlineEditableTextBlock> SaveFilenameEditBox;
	TSharedPtr<SInlineEditableTextBlock> NewDirectoryEditBox;
	TSharedPtr<SBox> SaveFilenameSizeBox;
	TSharedPtr<STextBlock> WindowTitle;
	TSharedPtr<SListView<TSharedPtr<FFileEntry>>> ListView;
	TSharedPtr<SBreadcrumbTrail<FString>> PathBreadcrumbTrail;

	TSharedPtr<SButton> NewDirCancelButton;
	TSharedPtr<SBox> NewDirectorySizeBox;
	TSharedPtr<STextBlock> DirErrorMsg;

	TArray<TSharedPtr<FString>> FilterNameArray;
	TArray<FString> FilterListArray;

	int32 FilterIndex;

	FSlateFileDlgWindow::EResult	 UserResponse;

	bool bNeedsBuilding;
	bool bRebuildDirPath;
	bool bDirectoryHasChanged;

	int32 DirNodeIndex;
	FString SaveFilename;

	TWeakPtr<SWindow> ParentWindow;
	FString CurrentPath;
	FString Filters;
	FString WindowTitleText;
	bool bMultiSelectEnabled;
	TArray<FString>* OutNames;
	int32* OutFilterIndex;
	bool bDirectoriesOnly;
	bool bSaveFile;
	FText AcceptText;
	FSlateFileDialogsStyle* StyleSet;

	IDirectoryWatcher *DirectoryWatcher;
	FDelegateHandle OnDialogDirectoryChangedDelegateHandle;
	FString RegisteredPath;
	FString NewDirectoryName;

	TArray<FString> History;
	int32 HistoryIndex;

	FDelegateHandle PostTickHandle;
};



//-----------------------------------------------------------------------------
class SSlateFileDialogRow : public SMultiColumnTableRow<SSlateFileDialogItemPtr>
{
public:
	
	SLATE_BEGIN_ARGS(SSlateFileDialogRow) { }
	SLATE_ARGUMENT(SSlateFileDialogItemPtr, DialogItem)
	SLATE_ARGUMENT(FSlateFileDialogsStyle *, StyleSet)
	SLATE_END_ARGS()


	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase> &InOwnerTableView)
	{
		check(InArgs._DialogItem.IsValid());
		
		DialogItem = InArgs._DialogItem;
		StyleSet = InArgs._StyleSet;
		
		SMultiColumnTableRow<SSlateFileDialogItemPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		FSlateFontInfo ItemFont = StyleSet->GetFontStyle("SlateFileDialogs.Dialog");
		struct EVisibility FolderIconVisibility = EVisibility::Visible;
		const FSlateBrush *Icon;
			
		if (DialogItem->bIsDirectory)
		{
			Icon = StyleSet->GetBrush("SlateFileDialogs.Folder16");
		}
		else
		{
			FString Extension = FPaths::GetExtension(DialogItem->Label, false);

			if (Extension.Equals(TEXT("uasset"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.UAsset16");
			}
			else if (Extension.Equals(TEXT("uproject"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.UProject16");
			}
			else if (Extension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.Model3D");
			}
			else if (Extension.Equals(TEXT("cpp"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("h"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("txt"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("log"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.TextFile");
			}
			else if (Extension.Equals(TEXT("wav"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("mp3"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("ogg"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.Audio");
			}
			else if (Extension.Equals(TEXT("mp4"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("webm"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.Video");
			}
			else if (Extension.Equals(TEXT("png"), ESearchCase::IgnoreCase) || 
					 Extension.Equals(TEXT("jpg"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("tga"), ESearchCase::IgnoreCase) ||
					 Extension.Equals(TEXT("bmp"), ESearchCase::IgnoreCase))
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.Image");
			}
			else
			{
				Icon = StyleSet->GetBrush("SlateFileDialogs.PlaceHolder");
				FolderIconVisibility = EVisibility::Hidden;
			}
		}

		if (ColumnName == TEXT("Pathname"))
		{
			return SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					.AutoWidth()
					.Padding(FMargin(5.0f, 2.0f))
					[
						SNew(SImage)
						.Image(Icon)
						.Visibility(FolderIconVisibility)
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					.AutoWidth()
					.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(FText::FromString(DialogItem->Label))
						.Font(ItemFont)
					];
		}
		else if (ColumnName == TEXT("ModDate"))
		{
			return SNew(STextBlock)
					.Text(FText::FromString(DialogItem->ModDate))
					.Font(ItemFont);
		}
		else if (ColumnName == TEXT("FileSize"))
		{
			return SNew(STextBlock)
					.Text(FText::FromString(DialogItem->FileSize))
					.Font(ItemFont);
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}	
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


private:
	SSlateFileDialogItemPtr DialogItem;
	FSlateFileDialogsStyle* StyleSet;
};

#undef LOCTEXT_NAMESPACE
