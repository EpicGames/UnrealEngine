// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for slate file dialog modules.
 */
class ISlateFileDialogsModule : public IModuleInterface
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~ISlateFileDialogsModule( ) = default;
	
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex = 0)
	{
		return false;
	}
	
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
	{
		int32 DummyFilterIndex;
		return OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyFilterIndex);
	}
	
	virtual bool OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		FString& OutFoldername)
	{
		return false;
	}
	
	virtual bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex = 0)
	{
		return false;
	}
	
	virtual bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
	{
		int32 DummyFilterIndex;
		return SaveFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyFilterIndex);
	}

	virtual ISlateFileDialogsModule* Get() { return nullptr; }
};
