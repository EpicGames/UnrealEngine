// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISlateFileDialogModule.h"

class FSlateFileDialogsStyle;

class FSlateFileDialogsModule : public ISlateFileDialogsModule
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	//ISlateFileDialogModule interface

	using ISlateFileDialogsModule::OpenFileDialog;
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex = 0) override;

	virtual bool OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		FString& OutFoldername) override;

	using ISlateFileDialogsModule::SaveFileDialog;
	virtual bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath,
		const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex,
		int32 DefaultFilterIndex = 0) override;

	virtual ISlateFileDialogsModule* Get() override;

	FSlateFileDialogsStyle *GetFileDialogsStyle() { return FileDialogStyle; }

private:
	ISlateFileDialogsModule *SlateFileDialog;

	FSlateFileDialogsStyle	*FileDialogStyle;
};
