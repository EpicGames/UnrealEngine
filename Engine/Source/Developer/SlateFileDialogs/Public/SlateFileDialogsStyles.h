// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateStyle.h"

class FSlateFileDialogsStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static const FSlateStyleSet *Get();
	static FName GetStyleSetName();

	static const FSlateBrush * GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleInstance->GetBrush(PropertyName, Specifier);
	}

	static FSlateFontInfo GetFontStyle(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleInstance->GetFontStyle(PropertyName, Specifier);
	}
	
private:
	static TSharedPtr<class FSlateStyleSet> Create();
	static TSharedPtr<class FSlateStyleSet> StyleInstance;
};
