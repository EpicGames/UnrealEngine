// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateFileDialogs : ModuleRules
{
    public SlateFileDialogs(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "InputCore",
                "Slate",
                "SlateCore",
                "DirectoryWatcher",
            }
        );

        PrivateIncludePathModuleNames.Add("TargetPlatform");
    }
}
